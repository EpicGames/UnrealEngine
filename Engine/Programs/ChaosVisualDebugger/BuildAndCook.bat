echo off
call "%~dp0..\..\Build\BatchFiles\RunUAT.bat" MakeCookedEditor -nop4 -utf8output -project="%~dp0/ChaosVisualDebugger.uproject" -DDC=noshared -stage -archive -package -build -pak -iostore -compressed -prereqs -archivedirectory="%~dp0PackagedBuild" -clientconfig="Development" -cook -nocompile -nocompileuat -AdditionalCookerOptions=" -CookProcessCount=4" -unrealexe="ChaosVisualDebugger.exe" %*
exit /b !ERRORLEVEL!