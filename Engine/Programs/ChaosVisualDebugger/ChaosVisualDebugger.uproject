{
	"FileVersion": 3,
	"EngineAssociation": "",
	"Category": "",
	"Description": "",
	"TargetPlatforms": [
		"WindowsCookedEditor"
	],
	"Plugins": [
		{
			"Name": "ChaosVD",
			"Enabled": true
		},
		{
			"Name": "PropertyAccessEditor",
			"Enabled": true
		},
		{
			"Name": "UdpMessaging",
			"Enabled": true
		}
	]
}