[/Script/Engine.GarbageCollectionSettings]
gc.MaxObjectsInProgram=500000
gc.MaxObjectsInGame=500000
gc.MaxObjectsInEditor=500000

[SystemSettingsEditor]
s.AllowUnversionedContentInEditor=1
cook.AllowCookedDataInEditorBuilds=1
s.EnforcePackageCompatibleVersionCheck=0

[SystemSettings]
s.AllowUnversionedContentInEditor=1
cook.AllowCookedDataInEditorBuilds=1
s.EnforcePackageCompatibleVersionCheck=0

[/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings]
bEnablePlugin=True
bAllowNetworkConnection=True
SecurityToken=D4CD916B4290446E57EEBCB1B9F9E4A2
bIncludeInShipping=False
bAllowExternalStartInShipping=False
bCompileAFSProject=False
bUseCompression=False
bLogFiles=False
bReportStats=False
ConnectionType=USBOnly
bUseManualIPAddress=False
ManualIPAddress=

[ConsoleVariables]
r.Nanite.ProjectEnabled=0
