[/Script/UnrealEd.ProjectPackagingSettings]
bUseZenStore=false

[Staging]
+DisallowedLocalizationTargets=Engine

[CookedEditorSettings]
bSupportCookedEditor=true
CookedEditorTargetName=ChaosVisualDebugger
bStageUAT=false
bStageShaderDirs=false
bStagePlatformBuildDirs=false
bStageExtrasDirs=false
bStagePlatformDirs=false
bIsForExternalDistribution=true
MapMode=None
+EngineExtraStageFiles=(Path="Plugins/ChaosVD/Resources", Recursive=false, NonUFS=true)
+EngineExtraStageFiles=(Path="Content/Splash", Recursive=false, NonUFS=true)

; remove everything in Platforms, except and Config files that aren't in Restricted, and shaders
+FinalStripFiles=(Filter="/Platforms/", Exceptions="(Engine)/Platforms/.*/Config || /Shaders/ || Android\.Automation || /XB1ZLib/")
+FinalStripFiles=(Filter="Engine/Extras")

; this will remove all of the compiled UAT script dlls, except for some ones that had their own subdirectories when built, that we need to keep
+FinalStripFiles=(Filter="AutomationTool/AutomationScripts/", Exceptions="/(BuildGraph|BuildPatchTool|ContentWorker|Scripts|Android)/")

